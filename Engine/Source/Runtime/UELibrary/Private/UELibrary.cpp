// Copyright Epic Games, Inc. All Rights Reserved.

#if defined(UE_LIBRARY_ENABLED) && UE_LIBRARY_ENABLED

#if !PLATFORM_WINDOWS || !defined(_WIN64)
	#error "UELibrary is currently only supported under 64-bit Windows"
#endif

#if !IS_MONOLITHIC
	#error "UELibrary is expected to be built monolithically"
#endif

#include "UELibrary.h"

#include "Windows/WindowsHWrapper.h"
#include "Containers/StringConv.h"

#define UELIBRARY_DLL_EXPORT
#include "UELibraryAPI.h"
#undef UELIBRARY_DLL_EXPORT

// From Launch
extern int32 LaunchWindowsStartup(HINSTANCE hInInstance, HINSTANCE hPrevInstance, char*, int32 nCmdShow, const TCHAR* CmdLine);
extern void LaunchWindowsShutdown();
extern void EngineTick();
extern void EngineExit();

// From ApplicationCore
extern LRESULT WindowsApplication_WndProc(HWND hWnd, UINT message, WPARAM wParam, LPARAM lParam);

namespace UE
{
namespace UELibrary
{
namespace Private
{
	bool LibraryIsInitialized = false;

	enum class EError : int
	{
		NoError,
		BadArguments,
		LibraryAlreadyInitialized,
		LibraryNotInitialized,
	};

	EError CheckInitPreconditions(HINSTANCE hInst, HWND MainWnd)
	{
		// Validate parameters
		if (!hInst || !MainWnd)
		{
			return EError::BadArguments;
		}

		// Validate that we haven't already been initialized
		if (LibraryIsInitialized)
		{
			return EError::LibraryAlreadyInitialized;
		}

		return EError::NoError;
	}

	void InitImpl(HINSTANCE hInst, HWND MainWnd, const TCHAR* CmdLine)
	{
		bool bHasGame = FParse::Param(CmdLine, TEXT("Game"));
		bool bHasWindowed = FParse::Param(CmdLine, TEXT("Windowed"));

		FString PatchedCmdLine;
		if (!bHasGame || !bHasWindowed)
		{
			PatchedCmdLine = CmdLine;
			if (!bHasGame)
			{
				PatchedCmdLine += TEXT(" -game");
			}
			if (!bHasWindowed)
			{
				PatchedCmdLine += TEXT(" -windowed");
			}
			CmdLine = *PatchedCmdLine;
		}

		*(HWND*)&GUELibraryOverrideSettings.WindowHandle = MainWnd;
		GUELibraryOverrideSettings.bIsEmbedded = true;

		LaunchWindowsStartup(hInst, nullptr, nullptr, 0, CmdLine);

		LibraryIsInitialized = true;
	}
}
}
} // namespace UE::UELibrary::Private

int UELibrary_InitA(HINSTANCE hInst, HWND MainWnd, const char* CmdLine)
{
	UE::UELibrary::Private::EError Result = UE::UELibrary::Private::CheckInitPreconditions(hInst, MainWnd);
	if (Result == UE::UELibrary::Private::EError::NoError)
	{
		FString ConvertedCmdLine = UTF8_TO_TCHAR(CmdLine);
		UE::UELibrary::Private::InitImpl(hInst, MainWnd, *ConvertedCmdLine);
	}

	return (int)Result;
}

int UELibrary_InitW(HINSTANCE hInst, HWND MainWnd, const wchar_t* CmdLine)
{
	UE::UELibrary::Private::EError Result = UE::UELibrary::Private::CheckInitPreconditions(hInst, MainWnd);
	if (Result == UE::UELibrary::Private::EError::NoError)
	{
		FString ConvertedCmdLine = StringCast<TCHAR>(CmdLine).Get();
		UE::UELibrary::Private::InitImpl(hInst, MainWnd, *ConvertedCmdLine);
	}

	return (int)Result;
}

int UELibrary_Tick()
{
	// Validate that we have already been initialized
	if (!UE::UELibrary::Private::LibraryIsInitialized)
	{
		return (int)UE::UELibrary::Private::EError::LibraryNotInitialized;
	}

	EngineTick();
	return 0;
}

LRESULT UELibrary_WndProc(HWND hWnd, UINT message, WPARAM wParam, LPARAM lParam)
{
	// Do default handling if the library isn't initialized
	if (!UE::UELibrary::Private::LibraryIsInitialized)
	{
		return DefWindowProc(hWnd, message, wParam, lParam);
	}
	else
	{
		return WindowsApplication_WndProc(hWnd, message, wParam, lParam);
	}
}

int UELibrary_Shutdown()
{
	// Validate that we have already been initialized
	if (!UE::UELibrary::Private::LibraryIsInitialized)
	{
		return (int)UE::UELibrary::Private::EError::LibraryNotInitialized;
	}

	EngineExit();
	LaunchWindowsShutdown();
	UE::UELibrary::Private::LibraryIsInitialized = false;
	UE::Trace::Shutdown();
	return 0;
}

#endif // #if defined(UE_LIBRARY_ENABLED) && UE_LIBRARY_ENABLED
