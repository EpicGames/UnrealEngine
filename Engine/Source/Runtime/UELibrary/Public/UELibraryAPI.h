// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#if PLATFORM_WINDOWS

#include "Windows/WindowsHWrapper.h"

#ifndef _WIN64
	#error "UELibrary is currently only supported under 64-bit Windows"
#endif

#ifdef UELIBRARY_DLL_EXPORT
	#define UELIBRARYAPI __declspec(dllexport)
#else
	#define UELIBRARYAPI __declspec(dllimport)
#endif

extern "C"
{
/**
 * Initializes UE as a library.
 *
 * @param  hInst    The instance of the outer application which wants to embed UE.
 * @param  hWnd     The window of the outer application into which UE is to be embedded.
 * @param  CmdLine  The command line to pass to UE - should contain a .uproject file and map to load at minimum.
 *
 * @return  Zero if creation was successful, non-zero if an error occurred.
 *
 * @note  UE can currently only be embedded in a single window.
 * @note  There is an A and W overload for different character widths of the command line argument. Typical usage
 *        should be to use the UELibrary_Init function which maps to the appropriate overload for the outer
 *        application's Unicode setting.
 */
UELIBRARYAPI int UELibrary_InitA(HINSTANCE hInst, HWND hWnd, const char* CmdLine);
UELIBRARYAPI int UELibrary_InitW(HINSTANCE hInst, HWND hWnd, const wchar_t* CmdLine);

#ifdef UNICODE
	#define UELibrary_Init UELibrary_InitW
#else
	#define UELibrary_Init UELibrary_InitA
#endif


/**
 * Ticks the UE library.  This should be called frequently by the outer application.
 *
 * @return  Zero if ticking was successful, non-zero if an error occurred.
 */
UELIBRARYAPI int UELibrary_Tick();


/**
 * Passes windows messages from the outer application to the UE library.
 *
 * @param  hWnd     As per WndProc.
 * @param  message  As per WndProc.
 * @param  wParam   As per WndProc.
 * @param  lParam   As per WndProc.
 *
 * @return  As per WndProc.
 */
UELIBRARYAPI LRESULT UELibrary_WndProc(HWND hWnd, UINT message, WPARAM wParam, LPARAM lParam);


/**
 * Shuts down the UE library.
 *
 * @return  Zero if shutdown was successful, non-zero if an error occurred.
 *
 * @note  UE cannot be started up again once shut down.
 */
UELIBRARYAPI int UELibrary_Shutdown();
}

#endif // PLATFORM_WINDOWS
