// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UELibrary : ModuleRules
{
	public UELibrary(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[] { "Core", "Engine", "InputCore" });

		// We depend on these modules, but can't explicitly state that for module ordering
		// reasons.  Instead, we extern those other modules' functions and call them directly,
		// which works because the UELibrary is only expected to be built as monolithic, so
		// there are no *_API issues.
		// PrivateDependencyModuleNames.AddRange(new string[] { "Launch", "ApplicationCore" });
	}
}
